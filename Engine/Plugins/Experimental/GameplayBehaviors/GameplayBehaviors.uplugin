{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "0.1",
	"FriendlyName" : "AI Behaviors",
	"Description" : "Encapsulated fire-and-forget behaviors for AI agents",
	"Category" : "Gameplay",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : true,
	"Installed" : false,
	"Modules": [
		{
			"Name": "GameplayBehaviorsModule",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "GameplayBehaviorsModule",
			"Type": "UncookedOnly",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "GameplayBehaviorsEditorModule",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit"
		}
	],
	"Plugins":
	[
		{
			"Name": "GameplayAbilities",
			"Enabled": true
		}
	]
}