// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayBehaviorsEditorModule.h"

#include "GameplayBehaviorsEditorStyle.h"

#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "ValueOrBBKeyDetails.h"

#define LOCTEXT_NAMESPACE "GameplayBehaviors"

class FGameplayBehaviorsEditorModule : public IGameplayBehaviorsEditorModule
{
protected:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

void FGameplayBehaviorsEditorModule::StartupModule()
{
	FGameplayBehaviorsEditorStyle::Get();

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomPropertyTypeLayout("ValueOrBBKey_GameplayTagContainer", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FValueOrBBKeyDetails_WithChild::MakeInstance));
	PropertyModule.NotifyCustomizationModuleChanged();
}

void FGameplayBehaviorsEditorModule::ShutdownModule()
{
	FGameplayBehaviorsEditorStyle::Shutdown();

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.UnregisterCustomPropertyTypeLayout("ValueOrBBKey_GameplayTagContainer");
	PropertyModule.NotifyCustomizationModuleChanged();
}

IMPLEMENT_MODULE(FGameplayBehaviorsEditorModule, GameplayBehaviorsEditorModule)

#undef LOCTEXT_NAMESPACE