// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayBehaviorsEditorStyle.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "Styling/SlateStyleRegistry.h"

TUniquePtr<FGameplayBehaviorsEditorStyle> FGameplayBehaviorsEditorStyle::Instance(nullptr);
FColor FGameplayBehaviorsEditorStyle::GameplayTagTypeColor(0, 26, 154);

FGameplayBehaviorsEditorStyle::FGameplayBehaviorsEditorStyle() : FSlateStyleSet("GameplayBehaviorsEditorStyle")
{
	Set("ClassIcon.BlackboardKeyType_GameplayTag", new FSlateRoundedBoxBrush(FLinearColor(GameplayTagTypeColor), 2.5f, FVector2D(16.f, 5.f)));
	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FGameplayBehaviorsEditorStyle::~FGameplayBehaviorsEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

FGameplayBehaviorsEditorStyle& FGameplayBehaviorsEditorStyle::Get()
{
	if (!Instance.IsValid())
	{
		Instance = TUniquePtr<FGameplayBehaviorsEditorStyle>(new FGameplayBehaviorsEditorStyle);
	}
	return *(Instance.Get());
}

void FGameplayBehaviorsEditorStyle::Shutdown()
{
	Instance.Reset();
}
