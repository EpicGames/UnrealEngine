// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class FGameplayBehaviorsEditorStyle final : public FSlateStyleSet
{
public:
	virtual ~FGameplayBehaviorsEditorStyle();

	static FGameplayBehaviorsEditorStyle& Get();
	static void Shutdown();

	static FColor GameplayTagTypeColor;

private:
	FGameplayBehaviorsEditorStyle();

	static TUniquePtr<FGameplayBehaviorsEditorStyle> Instance;
};
