// Copyright Epic Games, Inc. All Rights Reserved.

#include "AI/BTDecorator_GameplayTagQuery.h"
#include "AbilitySystemComponent.h"
#include "AbilitySystemGlobals.h"
#include "BehaviorTree/BlackboardComponent.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType_Object.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTDecorator_GameplayTagQuery)

struct FBTDecorator_GameplayTagQueryMemory
{
	TWeakObjectPtr<UAbilitySystemComponent> CachedAbilitySystemComponent;

	/** Array of handles for our gameplay tag query delegates */
	TArray<TTuple<FGameplayTag, FDelegateHandle>> GameplayTagEventHandles;
};

UBTDecorator_GameplayTagQuery::UBTDecorator_GameplayTagQuery(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	NodeName = "Gameplay Tag Query";

	INIT_DECORATOR_NODE_NOTIFY_FLAGS();

	// Accept only actors
	ActorForGameplayTagQuery.AddObjectFilter(this, GET_MEMBER_NAME_CHECKED(UBTDecorator_GameplayTagQuery, ActorForGameplayTagQuery), AActor::StaticClass());

	// Default to using Self Actor
	ActorForGameplayTagQuery.SelectedKeyName = FBlackboard::KeySelf;
}

bool UBTDecorator_GameplayTagQuery::CalculateRawConditionValue(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const
{
	const UBlackboardComponent* BlackboardComp = OwnerComp.GetBlackboardComponent();
	if (!BlackboardComp)
	{
		// Not calling super here since it returns true
		return false;
	}

	const IGameplayTagAssetInterface* GameplayTagAssetInterface = Cast<IGameplayTagAssetInterface>(BlackboardComp->GetValue<UBlackboardKeyType_Object>(ActorForGameplayTagQuery.GetSelectedKeyID()));
	if (!GameplayTagAssetInterface)
	{
		// Not calling super here since it returns true
		return false;
	}

	FGameplayTagContainer SelectedActorTags;
	GameplayTagAssetInterface->GetOwnedGameplayTags(SelectedActorTags);

	return GameplayTagQuery.Matches(SelectedActorTags);
}

void UBTDecorator_GameplayTagQuery::OnGameplayTagInQueryChanged(const FGameplayTag InTag, int32 NewCount, TWeakObjectPtr<UBehaviorTreeComponent> BehaviorTreeComponent)
{
	if (!BehaviorTreeComponent.IsValid())
	{
		return;
	}

	ConditionalFlowAbort(*BehaviorTreeComponent, EBTDecoratorAbortRequest::ConditionResultChanged);
}

FString UBTDecorator_GameplayTagQuery::GetStaticDescription() const
{
	return FString::Printf(TEXT("%s: %s"), *Super::GetStaticDescription(), *GameplayTagQuery.GetDescription());
}

void UBTDecorator_GameplayTagQuery::InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const
{
	InitializeNodeMemory<FBTDecorator_GameplayTagQueryMemory>(NodeMemory, InitType);
}

void UBTDecorator_GameplayTagQuery::CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const
{
	const FBTDecorator_GameplayTagQueryMemory* MyMemory = CastInstanceNodeMemory<FBTDecorator_GameplayTagQueryMemory>(NodeMemory);
	ensureMsgf(MyMemory->GameplayTagEventHandles.Num() == 0, TEXT("Dangling gameplay tag event handles for decorator %s"), *GetStaticDescription());
	CleanupNodeMemory<FBTDecorator_GameplayTagQueryMemory>(NodeMemory, CleanupType);
}

void UBTDecorator_GameplayTagQuery::OnBecomeRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	UBlackboardComponent* BlackboardComp = OwnerComp.GetBlackboardComponent();
	if (!BlackboardComp)
	{
		// Not calling super here since it does nothing
		return;
	}

	const AActor* SelectedActor = Cast<AActor>(BlackboardComp->GetValue<UBlackboardKeyType_Object>(ActorForGameplayTagQuery.GetSelectedKeyID()));
	if (!SelectedActor)
	{
		// Not calling super here since it does nothing
		return;
	}

	FBTDecorator_GameplayTagQueryMemory* MyMemory = CastInstanceNodeMemory<FBTDecorator_GameplayTagQueryMemory>(NodeMemory);
	MyMemory->CachedAbilitySystemComponent = UAbilitySystemGlobals::GetAbilitySystemComponentFromActor(SelectedActor);

	if (MyMemory->CachedAbilitySystemComponent.IsValid())
	{
		for (const FGameplayTag& CurrentTag : QueryTags)
		{
			FDelegateHandle GameplayTagEventCallbackDelegate = MyMemory->CachedAbilitySystemComponent.Get()->RegisterGameplayTagEvent(CurrentTag, EGameplayTagEventType::Type::AnyCountChange).AddUObject(this, &UBTDecorator_GameplayTagQuery::OnGameplayTagInQueryChanged, TWeakObjectPtr<UBehaviorTreeComponent>(&OwnerComp));
			MyMemory->GameplayTagEventHandles.Emplace(CurrentTag, GameplayTagEventCallbackDelegate);
		}
	}
}

void UBTDecorator_GameplayTagQuery::OnCeaseRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	FBTDecorator_GameplayTagQueryMemory* MyMemory = CastInstanceNodeMemory<FBTDecorator_GameplayTagQueryMemory>(NodeMemory);

	if (MyMemory->CachedAbilitySystemComponent.IsValid())
	{
		for (const TTuple<FGameplayTag, FDelegateHandle>& GameplayTagEvent : MyMemory->GameplayTagEventHandles)
		{
			MyMemory->CachedAbilitySystemComponent.Get()->RegisterGameplayTagEvent(GameplayTagEvent.Key, EGameplayTagEventType::Type::AnyCountChange).Remove(GameplayTagEvent.Value);
		}
	}

	MyMemory->GameplayTagEventHandles.Reset();
	MyMemory->CachedAbilitySystemComponent = nullptr;
}

uint16 UBTDecorator_GameplayTagQuery::GetInstanceMemorySize() const
{
	return sizeof(FBTDecorator_GameplayTagQueryMemory);
}

#if WITH_EDITOR
void UBTDecorator_GameplayTagQuery::CacheGameplayTagsInsideQuery()
{
	QueryTags.Reset();
	GameplayTagQuery.GetGameplayTagArray(QueryTags);
}

void UBTDecorator_GameplayTagQuery::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	if (PropertyChangedEvent.Property == NULL)
	{
		return;
	}

	CacheGameplayTagsInsideQuery();
}

FString UBTDecorator_GameplayTagQuery::GetErrorMessage() const
{
	if (GetBlackboardAsset() == nullptr)
	{
		return UE::BehaviorTree::Messages::BlackboardNotSet.ToString();
	}
	return Super::GetErrorMessage();
}
#endif	// WITH_EDITOR

void UBTDecorator_GameplayTagQuery::InitializeFromAsset(UBehaviorTree& Asset)
{
	Super::InitializeFromAsset(Asset);

	if (const UBlackboardData* BBAsset = GetBlackboardAsset())
	{
		ActorForGameplayTagQuery.ResolveSelectedKey(*BBAsset);
	}
	else
	{
		ActorForGameplayTagQuery.InvalidateResolvedKey();
	}
}
