// Copyright Epic Games, Inc. All Rights Reserved.

#include "AI/BTTask_SetKeyValueGameplayTag.h"

#include "BehaviorTree/BehaviorTreeComponent.h"
#include "BlackboardKeyType_GameplayTag.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTTask_SetKeyValueGameplayTag)

UBTTask_SetKeyValueGameplayTag::UBTTask_SetKeyValueGameplayTag(const FObjectInitializer& ObjectInitializer /*= FObjectInitializer::Get()*/)
{
	NodeName = "Set GameplayTag Key";
	BlackboardKey.AllowNoneAsValue(false);
	const FName FilterName = MakeUniqueObjectName(this, UBlackboardKeyType_GameplayTag::StaticClass(), *FString::Printf(TEXT("%s_GameplayTag"), *GET_MEMBER_NAME_CHECKED(UBTTask_SetKeyValueGameplayTag, BlackboardKey).ToString()));
	UBlackboardKeyType_GameplayTag* FilterOb = ObjectInitializer.CreateDefaultSubobject<UBlackboardKeyType_GameplayTag>(this, FilterName, /*Transient*/ true);
	BlackboardKey.AllowedTypes.Add(FilterOb);
	INIT_TASK_NODE_NOTIFY_FLAGS();
}

EBTNodeResult::Type UBTTask_SetKeyValueGameplayTag::ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	UBlackboardComponent* Blackboard = OwnerComp.GetBlackboardComponent();
	if (!Blackboard)
	{
		return EBTNodeResult::Failed;
	}

	if (BlackboardKey.GetSelectedKeyID() == FBlackboard::InvalidKey)
	{
		return EBTNodeResult::Failed;
	}

	Blackboard->SetValue<UBlackboardKeyType_GameplayTag>(BlackboardKey.GetSelectedKeyID(), Value.GetValue(*Blackboard));
	return EBTNodeResult::Succeeded;
}

FString UBTTask_SetKeyValueGameplayTag::GetStaticDescription() const
{
	return FString::Format(TEXT("Setting {0} to {1}"), { BlackboardKey.SelectedKeyName.ToString(), Value.ToString() });
}
