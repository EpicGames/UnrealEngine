// Copyright Epic Games, Inc. All Rights Reserved.

#include "AI/BTTask_StopGameplayBehavior.h"
#include "GameplayBehavior.h"
#include "GameplayBehaviorSubsystem.h"
#include "AIController.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BTTask_StopGameplayBehavior)

//----------------------------------------------------------------------//
//  UBTTask_StopGameplayBehavior
//----------------------------------------------------------------------//
UBTTask_StopGameplayBehavior::UBTTask_StopGameplayBehavior(const FObjectInitializer& ObjectInitializer)
{

}

EBTNodeResult::Type UBTTask_StopGameplayBehavior::ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory)
{
	UWorld* World = GetWorld();
	UGameplayBehaviorSubsystem* Subsystem = UGameplayBehaviorSubsystem::GetCurrent(World);
	AAIController* MyController = OwnerComp.GetAIOwner();
	if (Subsystem == nullptr || MyController == nullptr
		|| MyController->GetPawn() == nullptr)
	{
		return EBTNodeResult::Failed;
	}

	AActor& Avatar = *MyController->GetPawn();

	return Subsystem->StopBehavior(Avatar, BehaviorToStop)
		? EBTNodeResult::Succeeded
		: EBTNodeResult::Failed;
}

FString UBTTask_StopGameplayBehavior::GetStaticDescription() const
{
	FString Result;
	if (BehaviorToStop)
	{
		Result += FString::Printf(TEXT("Stop current gameplay behavior of type %s")
			, *BehaviorToStop->GetName());
	}
	else
	{
		Result += FString::Printf(TEXT("Stop any current gameplay behavior"));
	}

	return Result;
}


