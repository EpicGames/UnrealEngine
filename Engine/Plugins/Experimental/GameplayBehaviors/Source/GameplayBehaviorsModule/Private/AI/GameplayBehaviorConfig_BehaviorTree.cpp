// Copyright Epic Games, Inc. All Rights Reserved.

#include "AI/GameplayBehaviorConfig_BehaviorTree.h"
#include "AI/GameplayBehavior_BehaviorTree.h"
#include "BehaviorTree/BehaviorTree.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayBehaviorConfig_BehaviorTree)


UGameplayBehaviorConfig_BehaviorTree::UGameplayBehaviorConfig_BehaviorTree(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	BehaviorClass = UGameplayBehavior_BehaviorTree::StaticClass();
	bRevertToPreviousBTOnFinish = true;
}

UBehaviorTree* UGameplayBehaviorConfig_BehaviorTree::GetBehaviorTree() const 
{ 
	return BehaviorTree.IsPending()
		? BehaviorTree.LoadSynchronous()
		: BehaviorTree.Get();
}

