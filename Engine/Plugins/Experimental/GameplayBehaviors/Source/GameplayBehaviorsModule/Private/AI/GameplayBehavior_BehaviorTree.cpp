// Copyright Epic Games, Inc. All Rights Reserved.

#include "AI/GameplayBehavior_BehaviorTree.h"
#include "AI/GameplayBehaviorConfig_BehaviorTree.h"
#include "BehaviorTree/BehaviorTree.h"
#include "VisualLogger/VisualLogger.h"
#include "Blueprint/AIBlueprintHelperLibrary.h"
#include "AIController.h"
#include "BehaviorTree/BehaviorTreeComponent.h"
#include "BehaviorTree/BlackboardComponent.h"
#include "TimerManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayBehavior_BehaviorTree)

//----------------------------------------------------------------------//
// UGameplayBehavior_BehaviorTree
//----------------------------------------------------------------------//
UGameplayBehavior_BehaviorTree::UGameplayBehavior_BehaviorTree(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	InstantiationPolicy = EGameplayBehaviorInstantiationPolicy::ConditionallyInstantiate;
}

bool UGameplayBehavior_BehaviorTree::Trigger(AActor& InAvatar, const UGameplayBehaviorConfig* Config /* = nullptr*/, AActor* SmartObjectOwner /* = nullptr*/)
{
	const UGameplayBehaviorConfig_BehaviorTree* BTConfig = Cast<const UGameplayBehaviorConfig_BehaviorTree>(Config);
	if (BTConfig == nullptr)
	{
		UE_VLOG(&InAvatar, LogGameplayBehavior, Warning, TEXT("Failed to trigger behavior %s for %s due to Config being null"),
			*GetName(), *InAvatar.GetName());
		return false;
	}

	UBehaviorTree* BT = BTConfig->GetBehaviorTree();
	if (BT == nullptr)
	{
		UE_VLOG(&InAvatar, LogGameplayBehavior, Warning, TEXT("Failed to trigger behavior %s for %s due to Config->BehaviorTree being null"),
			*GetName(), *InAvatar.GetName());
		return false;
	}
	
	// note that the value stored in this property is unreliable if we're in the CDO
	// If you need this to be reliable set InstantiationPolicy to Instantiate
	AIController = UAIBlueprintHelperLibrary::GetAIController(&InAvatar);
	if (AIController == nullptr)
	{
		UE_VLOG(&InAvatar, LogGameplayBehavior, Warning, TEXT("Failed to trigger behavior %s due to %s not being AI-controlled"),
			*GetName(), *InAvatar.GetName());
		return false;
	}

	UBehaviorTreeComponent* BTComp = Cast<UBehaviorTreeComponent>(AIController->GetBrainComponent());
	if (BTComp == nullptr)
	{
		UE_VLOG(&InAvatar, LogGameplayBehavior, Warning, TEXT("Failed to trigger behavior %s due to %s missing a BehaviorTreeComponent"),
			*GetName(), *InAvatar.GetName());
		return false;
	}

	UBlackboardComponent* BlackboardComp = BTComp->GetBlackboardComponent();
	if (BlackboardComp == nullptr)
	{
		UE_VLOG(&InAvatar, LogGameplayBehavior, Warning, TEXT("Failed to trigger behavior %s due to %s missing a BlackboardComponent"),
			*GetName(), *InAvatar.GetName());
		return false;
	}
	
	PreviousBT = BTConfig->ShouldStorePreviousBT() ? BTComp->GetRootTree() : nullptr;
	if (UBlackboardData* BlackboardAsset = BT->GetBlackboardAsset())
	{
		BlackboardComp->InitializeBlackboard(*BlackboardAsset);
	}
	BTComp->StartTree(*BT, bSingleRun ? EBTExecutionMode::SingleRun : EBTExecutionMode::Looped);

	TimerHandle = InAvatar.GetWorld()->GetTimerManager().SetTimerForNextTick(this, &UGameplayBehavior_BehaviorTree::OnTimerTick);
	
	// BrainComponent might be Paused due to AIResource locking. Keep lock but resume logic.
	if (BTComp->IsPaused())
	{
		BTComp->ResumeLogic(TEXT("Allow inner BT to run within GameplayBehavior_BehaviorTree"));
	}

	AIController->OnPossessedPawnChanged.AddDynamic(this, &ThisClass::OnPossessedPawnChanged);

	return true;
}

void UGameplayBehavior_BehaviorTree::OnTimerTick()
{
	if (!ensureMsgf(IsValid(AIController), TEXT("Tick should not have been registered for an invalid AIController")))
	{
		return;
	}

	const UBehaviorTreeComponent* BTComp = Cast<UBehaviorTreeComponent>(AIController->GetBrainComponent());
	if (!ensureMsgf(IsValid(BTComp), TEXT("Tick should not have been registered for an AIController without a valid BehaviorTreeComponent")))
	{
		return;
	}
	
	if (!BTComp->IsRunning())
	{
		AActor* Pawn = AIController->GetPawn();
		if (ensureMsgf(IsValid(Pawn), TEXT("A valid Pawn is required to end the behavior")))
		{
			EndBehavior(*Pawn, /*bInterrupted*/false);
		}
	}
	else
	{
		TimerHandle = AIController->GetWorld()->GetTimerManager().SetTimerForNextTick(this, &UGameplayBehavior_BehaviorTree::OnTimerTick);		
	}
}

void UGameplayBehavior_BehaviorTree::OnPossessedPawnChanged(APawn* OldPawn, APawn*)
{
	if (OldPawn != nullptr)
	{
		EndBehavior(*OldPawn, /*bInterrupted*/true);
	}
}

void UGameplayBehavior_BehaviorTree::EndBehavior(AActor& InAvatar, const bool bInterrupted)
{
	Super::EndBehavior(InAvatar, bInterrupted);

	if (AIController)
	{
		AIController->OnPossessedPawnChanged.RemoveAll(this);

		if (bInterrupted)
		{
			if (UBehaviorTreeComponent* BTComp = Cast<UBehaviorTreeComponent>(AIController->GetBrainComponent()))
			{
				BTComp->StopTree(EBTStopMode::Safe);
			}
		}

		if (PreviousBT)
		{
			AIController->RunBehaviorTree(PreviousBT);
		}
	}
}

bool UGameplayBehavior_BehaviorTree::NeedsInstance(const UGameplayBehaviorConfig* Config) const
{
	const UGameplayBehaviorConfig_BehaviorTree* BTConfig = Cast<const UGameplayBehaviorConfig_BehaviorTree>(Config);
	return BTConfig && BTConfig->ShouldStorePreviousBT();
}

