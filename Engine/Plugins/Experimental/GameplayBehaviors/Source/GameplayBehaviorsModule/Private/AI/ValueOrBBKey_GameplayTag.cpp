// Copyright Epic Games, Inc. All Rights Reserved.
#include "AI/ValueOrBBKey_GameplayTag.h"

#include "BlackboardKeyType_GameplayTag.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ValueOrBBKey_GameplayTag)

FString FValueOrBBKey_GameplayTagContainer::ToString() const
{
	if (!Key.IsNone())
	{
		return ToStringKeyName();
	}
	else
	{
		return DefaultValue.ToStringSimple(false);
	}
}

FGameplayTagContainer FValueOrBBKey_GameplayTagContainer::GetValue(const UBlackboardComponent& Blackboard) const
{
	return FBlackboard::GetValue<UBlackboardKeyType_GameplayTag>(Blackboard, Key, KeyId, DefaultValue);
}

FGameplayTagContainer FValueOrBBKey_GameplayTagContainer::GetValue(const UBlackboardComponent* Blackboard) const
{
	return Blackboard ? GetValue(*Blackboard) : DefaultValue;
}

FGameplayTagContainer FValueOrBBKey_GameplayTagContainer::GetValue(const UBehaviorTreeComponent& BehaviorComp) const
{
	return FBlackboard::GetValue<UBlackboardKeyType_GameplayTag>(BehaviorComp, Key, KeyId, DefaultValue);
}

FGameplayTagContainer FValueOrBBKey_GameplayTagContainer::GetValue(const UBehaviorTreeComponent* BehaviorComp) const
{
	return BehaviorComp ? GetValue(*BehaviorComp) : DefaultValue;
}

bool FValueOrBBKey_GameplayTagContainer::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.GetType().IsStruct(FGameplayTagContainer::StaticStruct()->GetFName()))
	{
		FGameplayTagContainer::StaticStruct()->SerializeItem(Slot, &DefaultValue, nullptr);
		return true;
	}
	return false;
}

FGameplayTagContainer UValueOrBBKey_GameplayTagBlueprintUtility::GetTagContainer(const FValueOrBBKey_GameplayTagContainer& Value, const UBehaviorTreeComponent* BehaviorTreeComp)
{
	return Value.GetValue(BehaviorTreeComp);
}

#if WITH_EDITOR
bool FValueOrBBKey_GameplayTagContainer::IsCompatibleType(const UBlackboardKeyType* KeyType) const
{
	return KeyType && KeyType->GetClass() == UBlackboardKeyType_GameplayTag::StaticClass();
}
#endif // WITH_EDITOR
