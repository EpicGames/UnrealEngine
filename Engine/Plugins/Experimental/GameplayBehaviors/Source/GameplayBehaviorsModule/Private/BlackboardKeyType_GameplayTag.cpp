// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlackboardKeyType_GameplayTag.h"
#include "GameplayTagContainer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlackboardKeyType_GameplayTag)

const UBlackboardKeyType_GameplayTag::FDataType UBlackboardKeyType_GameplayTag::InvalidValue = FGameplayTagContainer::EmptyContainer;

UBlackboardKeyType_GameplayTag::UBlackboardKeyType_GameplayTag(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	ValueSize = sizeof(FGameplayTagContainer);
	SupportedOp = EBlackboardKeyOperation::Text;
}

FGameplayTagContainer UBlackboardKeyType_GameplayTag::GetValue(const UBlackboardKeyType_GameplayTag* KeyOb, const uint8* RawData)
{
	return GetValueFromMemory<FGameplayTagContainer>(RawData);
}

bool UBlackboardKeyType_GameplayTag::SetValue(UBlackboardKeyType_GameplayTag* KeyOb, uint8* RawData, const FGameplayTagContainer& Value)
{
	return SetValueInMemory<FGameplayTagContainer>(RawData, Value);
}

EBlackboardCompare::Type UBlackboardKeyType_GameplayTag::CompareValues(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock,
	const UBlackboardKeyType* OtherKeyOb, const uint8* OtherMemoryBlock) const
{
	const FGameplayTagContainer MyValue = GetValue(this, MemoryBlock);
	const FGameplayTagContainer OtherValue = GetValue((UBlackboardKeyType_GameplayTag*)OtherKeyOb, OtherMemoryBlock);

	return (MyValue.HasAll(OtherValue) ? EBlackboardCompare::Equal : EBlackboardCompare::NotEqual);
}

FString UBlackboardKeyType_GameplayTag::DescribeValue(const UBlackboardComponent& OwnerComp, const uint8* RawData) const
{
	const FGameplayTagContainer CurrentValue = GetValue(this, RawData);
	if (CurrentValue.IsEmpty())
	{
		static const FString EmptyContainer = TEXT("Empty");
		return EmptyContainer;
	}
	return CurrentValue.ToStringSimple();
}

bool UBlackboardKeyType_GameplayTag::TestTextOperation(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock, ETextKeyOperation::Type Op, const FString& OtherString) const
{
	const FString Value = GetValue(this, MemoryBlock).ToStringSimple();
	switch (Op)
	{
	case ETextKeyOperation::Equal:			return (Value == OtherString);
	case ETextKeyOperation::NotEqual:		return (Value != OtherString);
	case ETextKeyOperation::Contain:		return (Value.Contains(OtherString) == true);
	case ETextKeyOperation::NotContain:		return (Value.Contains(OtherString) == false);
	default: break;
	}

	return false;
}

void UBlackboardKeyType_GameplayTag::CopyValues(UBlackboardComponent& OwnerComp, uint8* MemoryBlock, const UBlackboardKeyType* SourceKeyOb, const uint8* SourceBlock)
{
	const UBlackboardKeyType_GameplayTag* GameplayTagSourceKeyOb = Cast<const UBlackboardKeyType_GameplayTag>(SourceKeyOb);
	if (GameplayTagSourceKeyOb != nullptr)
	{
		SetValue(this, MemoryBlock, GetValue(GameplayTagSourceKeyOb, SourceBlock));
	}
}

void UBlackboardKeyType_GameplayTag::InitializeMemory(UBlackboardComponent& OwnerComp, uint8* MemoryBlock)
{
	SetValue(this, MemoryBlock, DefaultValue);
}

void UBlackboardKeyType_GameplayTag::Clear(UBlackboardComponent& OwnerComp, uint8* MemoryBlock)
{
	SetValue(this, MemoryBlock, InvalidValue);
}

bool UBlackboardKeyType_GameplayTag::IsEmpty(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock) const
{
	const FDataType TagContainer = GetValue(this, MemoryBlock);

	return TagContainer.IsEmpty();
}
