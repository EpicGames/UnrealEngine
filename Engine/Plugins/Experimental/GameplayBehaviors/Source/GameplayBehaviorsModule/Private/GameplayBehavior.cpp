// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayBehavior.h"
#include "AIController.h"
#include "GameFramework/Character.h"
#include "Tasks/AITask.h"
#include "VisualLogger/VisualLogger.h"
#include "AbilitySystemInterface.h"
#include "AbilitySystemComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayBehavior)


DEFINE_LOG_CATEGORY(LogGameplayBehavior);


UGameplayBehavior::UGameplayBehavior(const FObjectInitializer& ObjectInitializer)
	: Super()
{
	bTransientIsTriggering = false;
	bTransientIsActive = false;
	bTransientIsEnding = false;
	TransientProps = 0;
	InstantiationPolicy = EGameplayBehaviorInstantiationPolicy::Instantiate;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		UClass* MyClass = GetClass();
		if (ensure(MyClass) && MyClass->HasAnyClassFlags(CLASS_CompiledFromBlueprint))
		{
			static const FName FuncNameOnTriggered = TEXT("K2_OnTriggered");
			static const FName FuncNameOnTriggeredPawn = TEXT("K2_OnTriggeredPawn");
			static const FName FuncNameOnTriggeredCharacter = TEXT("K2_OnTriggeredCharacter");
			static const FName FuncNameOnFinished = TEXT("K2_OnFinished");
			static const FName FuncNameOnFinishedPawn = TEXT("K2_OnFinishedPawn");
			static const FName FuncNameOnFinishedCharacter = TEXT("K2_OnFinishedCharacter");

			bTriggerGeneric = bTriggerGeneric || (MyClass->FindFunctionByName(FuncNameOnTriggered) != nullptr);
			bTriggerPawn = bTriggerPawn || (MyClass->FindFunctionByName(FuncNameOnTriggeredPawn) != nullptr);
			bTriggerCharacter = bTriggerCharacter || (MyClass->FindFunctionByName(FuncNameOnTriggeredCharacter) != nullptr);
			bFinishedGeneric = bFinishedGeneric || (MyClass->FindFunctionByName(FuncNameOnFinished) != nullptr);
			bFinishedPawn = bFinishedPawn || (MyClass->FindFunctionByName(FuncNameOnFinishedPawn) != nullptr);
			bFinishedCharacter = bFinishedCharacter || (MyClass->FindFunctionByName(FuncNameOnFinishedCharacter) != nullptr);
		}
	}
}

void UGameplayBehavior::PostInitProperties()
{
	Super::PostInitProperties();

	const UGameplayBehavior* CDO = GetClass()->GetDefaultObject<UGameplayBehavior>();
	TransientProps = CDO->TransientProps;
}

void UGameplayBehavior::BeginDestroy()
{
	Super::BeginDestroy();
}

UWorld* UGameplayBehavior::GetWorld() const
{
	// Currently Gameplay Behaviors are created with the World as their outer, so for instances this should always work
	if (const UObject* Outer = GetOuter())
	{
		if (UWorld* World = Outer->GetWorld())
		{
			return World;
		}
	}

	// The CDO has no World outer, so let's take the World from the affected actor,
	// which will be available in all BP Trigger events
	if (TransientAvatar)
	{
		return TransientAvatar->GetWorld();
	}

	// Do not call super, as that'll indicate to the BP editor that GetWorld() is not implemented,
	// and will result in an error that calling world-dependent functions is unsafe
	return nullptr;
}

bool UGameplayBehavior::Trigger(AActor& Avatar, const UGameplayBehaviorConfig* Config, AActor* SmartObjectOwner/* = nullptr*/)
{
	bTransientIsTriggering = true;

	TransientAvatar = &Avatar;
	TransientSmartObjectOwner = SmartObjectOwner;

	if (bTriggerGeneric || bTriggerPawn || bTriggerCharacter)
	{
		// most common case, that's why we consider it first. Plus it's most specific
		ACharacter* CharacterAvatar = bTriggerCharacter ? Cast<ACharacter>(&Avatar) : nullptr;
		if (CharacterAvatar)
		{
			bTransientIsActive = true;
			K2_OnTriggeredCharacter(CharacterAvatar, Config, SmartObjectOwner);
		}
		else if (bTriggerGeneric || bTriggerPawn)
		{
			APawn* PawnAvatar = bTriggerPawn ? Cast<APawn>(&Avatar) : nullptr;
			if (PawnAvatar)
			{
				bTransientIsActive = true;
				K2_OnTriggeredPawn(PawnAvatar, Config, SmartObjectOwner);
			}
			else if (bTriggerGeneric)
			{
				bTransientIsActive = true;
				K2_OnTriggered(&Avatar, Config, SmartObjectOwner);
			}
		}
	}

	bTransientIsTriggering = false;

	// bTransientIsActive might get changed by BP "end behavior" calls so we need to 
	// detect the behavior has finished synchronously and inform the caller	
	return bTransientIsActive;
}

void UGameplayBehavior::EndBehavior(AActor& Avatar, const bool bInterrupted)
{
	if (bTransientIsEnding)
	{
		return;
	}
	bTransientIsEnding = true;

	// @todo handle CDOs
	// Tell all our tasks that we are finished and they should cleanup
	for (int32 TaskIdx = ActiveTasks.Num() - 1; TaskIdx >= 0 && ActiveTasks.Num() > 0; --TaskIdx)
	{
		UGameplayTask* Task = ActiveTasks[TaskIdx];
		if (Task)
		{
			Task->TaskOwnerEnded();
		}
	}
	ActiveTasks.Reset();	// Empty the array but dont resize memory, since this object is probably going to be destroyed very soon anyways.

	if (bFinishedGeneric || bFinishedPawn || bFinishedCharacter)
	{
		// most common case, that's why we consider it first. Plus it's most specific
		ACharacter* CharacterAvatar = bFinishedGeneric ? Cast<ACharacter>(&Avatar) : nullptr;
		if (CharacterAvatar)
		{
			K2_OnFinishedCharacter(CharacterAvatar, bInterrupted);
		}
		else if (bFinishedGeneric || bFinishedPawn)
		{
			APawn* PawnAvatar = bFinishedPawn ? Cast<APawn>(&Avatar) : nullptr;
			if (PawnAvatar)
			{
				K2_OnFinishedPawn(PawnAvatar, bInterrupted);
			}
			else if (bFinishedGeneric)
			{
				K2_OnFinished(&Avatar, bInterrupted);
			}
		}
	}

	// if bTransientIsTriggering is true it means we're in the middle of
	// triggering this behavior, so there's no point in sending 'on done' notifies 
	if (bTransientIsTriggering == false)
	{
		OnBehaviorFinished.Broadcast(*this, Avatar, bInterrupted);
	}

	bTransientIsEnding = false;
	bTransientIsActive = false;	
}

TOptional<FVector> UGameplayBehavior::GetDynamicLocation(const AActor* InAvatar, const UGameplayBehaviorConfig* InConfig, const AActor* InSmartObjectOwner) const
{
	return TOptional<FVector>();
}

UGameplayTasksComponent* UGameplayBehavior::GetGameplayTasksComponent(const UGameplayTask& Task) const
{
	// @todo this is a part where the behavior can break for human players
	const UAITask* AITask = Cast<UAITask>(&Task);
	if (AITask)
	{
		return AITask->GetAIController() ? AITask->GetAIController()->GetGameplayTasksComponent() : nullptr;
	}

	if (TransientAvatar)
	{
		APawn* AsPawn = Cast<APawn>(TransientAvatar);
		if (AsPawn != nullptr)
		{
			AAIController* AIController = Cast<AAIController>(AsPawn->GetController());
			if (AIController != nullptr)
			{
				return AIController->GetGameplayTasksComponent();
			}
		}

		IAbilitySystemInterface* AsAbilitySysInterface = Cast<IAbilitySystemInterface>(TransientAvatar);
		if (AsAbilitySysInterface)
		{
			return AsAbilitySysInterface->GetAbilitySystemComponent();
		}

		IGameplayTaskOwnerInterface* AsTaskOwnerInterface = Cast<IGameplayTaskOwnerInterface>(TransientAvatar);
		if (AsTaskOwnerInterface)
		{
			return AsTaskOwnerInterface->GetGameplayTasksComponent(Task);
		}
	}
	return nullptr;
}

AActor* UGameplayBehavior::GetGameplayTaskOwner(const UGameplayTask* Task) const
{
	// @todo this is a part where the behavior can break for human players
	const UAITask* AITask = Cast<UAITask>(Task);
	return AITask ? AITask->GetAIController() : TransientAvatar;
}

AActor* UGameplayBehavior::GetGameplayTaskAvatar(const UGameplayTask* Task) const
{
	if (TransientAvatar)
	{
		return TransientAvatar;
	}

	const UAITask* AITask = Cast<UAITask>(Task);
	return AITask && AITask->GetAIController() ? AITask->GetAIController()->GetPawn() : nullptr;
}

void UGameplayBehavior::OnGameplayTaskActivated(UGameplayTask& Task)
{
	UE_VLOG(GetOuter(), LogGameplayBehavior, Log, TEXT("Behavior %s Task STARTED %s")
		, *GetName(), *Task.GetName());

	ActiveTasks.Add(&Task);
}

void UGameplayBehavior::OnGameplayTaskDeactivated(UGameplayTask& Task)
{
	UE_VLOG(GetOuter(), LogGameplayBehavior, Log, TEXT("Behavior %s Task ENDED %s")
		, *GetName(), *Task.GetName());

	ActiveTasks.Remove(&Task);
}

void UGameplayBehavior::K2_EndBehavior(AActor* Avatar)
{
	if (Avatar)
	{
		EndBehavior(*Avatar);
	}
}

void UGameplayBehavior::K2_AbortBehavior(AActor* Avatar)
{
	if (Avatar)
	{
		AbortBehavior(*Avatar);
	}
}

void UGameplayBehavior::K2_TriggerBehavior(AActor* Avatar, UGameplayBehaviorConfig* Config /* = nullptr*/, AActor* SmartObjectOwner /* = nullptr*/)
{
	if (Avatar)
	{
		Trigger(*Avatar, Config, SmartObjectOwner);
	}
}

int32 UGameplayBehavior::K2_GetNextActorIndexInSequence(int32 CurrentIndex) const
{
	// starting from 1 to not end up picking CurrentIndex
	for (int32 Iteration = 1; Iteration < RelevantActors.Num(); ++Iteration)
	{
		const int32 Index = (Iteration + CurrentIndex) % RelevantActors.Num();
		if (RelevantActors[Index])
		{
			return Index;
		}
	}

	return INDEX_NONE;
}


