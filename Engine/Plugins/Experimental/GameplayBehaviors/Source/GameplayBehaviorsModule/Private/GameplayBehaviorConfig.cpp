// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayBehaviorConfig.h"
#include "GameplayBehavior.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayBehaviorConfig)


UGameplayBehavior* UGameplayBehaviorConfig::GetBehavior(UWorld& World) const
{
	if (!BehaviorClass)
	{
		return nullptr;
	}

	UGameplayBehavior* BehaviorCDO = GetMutableDefault<UGameplayBehavior>(BehaviorClass);

	return (BehaviorCDO && BehaviorCDO->IsInstanced(this))
		? NewObject<UGameplayBehavior>(&World, BehaviorClass)
		: BehaviorCDO;
}

