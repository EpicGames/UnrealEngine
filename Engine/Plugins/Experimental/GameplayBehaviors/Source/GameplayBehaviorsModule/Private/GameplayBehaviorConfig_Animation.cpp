// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayBehaviorConfig_Animation.h"
#include "GameplayBehavior_AnimationBased.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayBehaviorConfig_Animation)


UGameplayBehaviorConfig_Animation::UGameplayBehaviorConfig_Animation(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	BehaviorClass = UGameplayBehavior_AnimationBased::StaticClass();
}

