// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayBehaviorSubsystem.h"

#include "Engine/World.h"
#include "GameplayBehaviorConfig.h"
#include "GameplayBehavior.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayBehaviorSubsystem)


bool UGameplayBehaviorSubsystem::StopBehavior(AActor& Avatar, const TSubclassOf<UGameplayBehavior> BehaviorToStop)
{
	FAgentGameplayBehaviors* AgentData = AgentGameplayBehaviors.Find(&Avatar);

	if (AgentData)
	{
		for (int32 Index = AgentData->Behaviors.Num() - 1; Index >= 0; --Index)
		{
			UGameplayBehavior* Beh = AgentData->Behaviors[Index];
			// @todo make sure we're aware of this in OnBehaviorFinished
			if (Beh && (!BehaviorToStop || Beh->IsA(BehaviorToStop)))
			{
				Beh->EndBehavior(Avatar, /*bInterrupted=*/true);
			}
		}
	}

	return false;
}

void UGameplayBehaviorSubsystem::OnBehaviorFinished(UGameplayBehavior& Behavior, AActor& Avatar, const bool bInterrupted)
{
	FAgentGameplayBehaviors* AgentData = AgentGameplayBehaviors.Find(&Avatar);

	if (AgentData)
	{
		const int32 BehaviorIndex = AgentData->Behaviors.Find(&Behavior);
		if (BehaviorIndex != INDEX_NONE)
		{
			Behavior.GetOnBehaviorFinishedDelegate().RemoveAll(this);
			AgentData->Behaviors.RemoveAtSwap(BehaviorIndex, EAllowShrinking::No);
		}
	}
}

bool UGameplayBehaviorSubsystem::TriggerBehavior(const UGameplayBehaviorConfig& Config, AActor& Avatar, AActor* SmartObjectOwner/* = nullptr*/)
{
	UWorld* World = Avatar.GetWorld();
	UGameplayBehavior* Behavior = World ? Config.GetBehavior(*World) : nullptr;
	return Behavior != nullptr && TriggerBehavior(*Behavior, Avatar, &Config, SmartObjectOwner);
}

bool UGameplayBehaviorSubsystem::TriggerBehavior(UGameplayBehavior& Behavior, AActor& Avatar, const UGameplayBehaviorConfig* Config, AActor* SmartObjectOwner/* = nullptr*/)
{
	UGameplayBehaviorSubsystem* Subsystem = GetCurrent(Avatar.GetWorld());
	return Subsystem != nullptr && Subsystem->TriggerBehaviorImpl(Behavior, Avatar, Config, SmartObjectOwner);
}

bool UGameplayBehaviorSubsystem::TriggerBehaviorImpl(UGameplayBehavior& Behavior, AActor& Avatar, const UGameplayBehaviorConfig* Config, AActor* SmartObjectOwner/* = nullptr*/)
{
	if (Behavior.Trigger(Avatar, Config, SmartObjectOwner))
	{
		Behavior.GetOnBehaviorFinishedDelegate().AddUObject(this, &UGameplayBehaviorSubsystem::OnBehaviorFinished);
		
		FAgentGameplayBehaviors& AgentData = AgentGameplayBehaviors.FindOrAdd(&Avatar);
		AgentData.Behaviors.Add(&Behavior);

		return true;
	}
	return false;
}

UGameplayBehaviorSubsystem* UGameplayBehaviorSubsystem::GetCurrent(const UWorld* World)
{
	return UWorld::GetSubsystem<UGameplayBehaviorSubsystem>(World);
}

