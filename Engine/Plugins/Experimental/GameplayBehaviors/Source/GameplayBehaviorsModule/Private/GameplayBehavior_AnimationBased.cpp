// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayBehavior_AnimationBased.h"
#include "GameplayBehaviorConfig_Animation.h"
#include "VisualLogger/VisualLogger.h"
#include "AbilitySystemComponent.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayBehavior_AnimationBased)

//----------------------------------------------------------------------//
// UGameplayBehavior_AnimationBased
//----------------------------------------------------------------------//
UGameplayBehavior_AnimationBased::UGameplayBehavior_AnimationBased(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

bool UGameplayBehavior_AnimationBased::Trigger(AActor& InAvatar, const UGameplayBehaviorConfig* Config /* = nullptr*/, AActor* SmartObjectOwner /* = nullptr*/)
{
	const UGameplayBehaviorConfig_Animation* AnimConfig = Cast<const UGameplayBehaviorConfig_Animation>(Config);
	if (AnimConfig == nullptr || AnimConfig->GetMontage() == nullptr)
	{
		UE_VLOG(&InAvatar, LogGameplayBehavior, Log, TEXT("Failed to trigger behavior %s due to %s being null")
			, *InAvatar.GetName(), AnimConfig ? TEXT("Config->Montage") : TEXT("Config"));
		return false;
	}

	return PlayMontage(InAvatar, *AnimConfig->GetMontage(), AnimConfig->GetPlayRate(), AnimConfig->GetStartSectionName(), AnimConfig->IsLooped());
}

bool UGameplayBehavior_AnimationBased::PlayMontage(AActor& InAvatar, UAnimMontage& AnimMontage, const float InPlayRate, const FName StartSectionName, const bool bLoop)
{
	bool bSuccess = false;
	UAbilitySystemComponent* ASC = Cast<UAbilitySystemComponent>(InAvatar.FindComponentByClass(UAbilitySystemComponent::StaticClass()));

	if (ASC)
	{
		FGameplayAbilityActivationInfo ActivationInfo(&InAvatar);
		ActivationInfo.bCanBeEndedByOtherInstance = true;
		const float PlaybackLength = ASC->PlayMontage(/*InAnimatingAbility=*/nullptr, ActivationInfo, &AnimMontage, InPlayRate, StartSectionName);
		if (PlaybackLength > 0)
		{
			FMontagePlaybackData* PlaybackData = ActivePlayback.FindByPredicate([&](const FMontagePlaybackData& Entry) { return Entry == &InAvatar; });
			if (PlaybackData == nullptr)
			{
				PlaybackData = &ActivePlayback.Add_GetRef(FMontagePlaybackData(InAvatar, AnimMontage, InPlayRate, StartSectionName, bLoop));
			}
			check(PlaybackData);
			PlaybackData->AbilityComponent = ASC;

			//UAnimInstance* AnimInstance = AbilityActorInfo.IsValid() ? AbilityActorInfo->GetAnimInstance() : nullptr;

			UWorld* World = InAvatar.GetWorld();
			if (World)
			{
				PlaybackData->TimerDelegate = FTimerDelegate::CreateUObject(this, &UGameplayBehavior_AnimationBased::OnMontageFinished, &AnimMontage, false, &InAvatar);

				World->GetTimerManager().SetTimer(PlaybackData->TimerHandle, PlaybackData->TimerDelegate, PlaybackLength, /*bLoop=*/false);
			}

			bSuccess = true;
		}
	}

	return bSuccess;
}

void UGameplayBehavior_AnimationBased::EndBehavior(AActor& InAvatar, const bool bInterrupted)
{
	if (bInterrupted)
	{
		// if interrupted we need to see if we're still playing a montage, and if so, abort it
		// also make sure we don't loop
		FMontagePlaybackData* PlaybackData = ActivePlayback.FindByPredicate([&](const FMontagePlaybackData& Entry) { return Entry == &InAvatar; });
		if (PlaybackData)
		{
			UWorld* World = InAvatar.GetWorld();
			if (World && PlaybackData->TimerHandle.IsValid())
			{
				World->GetTimerManager().ClearTimer(PlaybackData->TimerHandle);
			}
			if (PlaybackData->AbilityComponent && PlaybackData->AnimMontage)
			{
				PlaybackData->AbilityComponent->StopMontageIfCurrent(*PlaybackData->AnimMontage);
			}
			ActivePlayback.RemoveSingleSwap(*PlaybackData, EAllowShrinking::No);
		}
	}

	Super::EndBehavior(InAvatar, bInterrupted);
}

void UGameplayBehavior_AnimationBased::OnMontageFinished(UAnimMontage* Montage, bool bInterrupted, AActor* InAvatar)
{
	FMontagePlaybackData* PlaybackData = ActivePlayback.FindByPredicate([&](const FMontagePlaybackData& Entry) { return Entry == InAvatar; });
	UE_LOGF(LogGameplayBehavior, Error, "Added log!");

	if (PlaybackData != nullptr)
	{
		check(Montage && InAvatar);

		if (bInterrupted == true || PlaybackData->bLoop == false
			|| PlaybackData->AbilityComponent == nullptr)
		{
			ActivePlayback.RemoveSingleSwap(*PlaybackData, EAllowShrinking::No);
			EndBehavior(*InAvatar, bInterrupted);
		}
		else
		{
			// request another playback
			FGameplayAbilityActivationInfo ActivationInfo(InAvatar);
			ActivationInfo.bCanBeEndedByOtherInstance = true;
			const float PlaybackLength = PlaybackData->AbilityComponent->PlayMontage(/*InAnimatingAbility=*/nullptr, ActivationInfo, Montage, PlaybackData->PlayRate, PlaybackData->SectionName);
			
			if (PlaybackLength > 0)
			{
				//UAnimInstance* AnimInstance = AbilityActorInfo.IsValid() ? AbilityActorInfo->GetAnimInstance() : nullptr;

				UWorld* World = InAvatar->GetWorld();
				if (World)
				{
					World->GetTimerManager().SetTimer(PlaybackData->TimerHandle, PlaybackData->TimerDelegate, PlaybackLength, /*bLoop=*/false);
				}
			}
		}
	}
}

