// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayBehaviorsBlueprintFunctionLibrary.h"
#include "BehaviorTree/BlackboardComponent.h"
#include "BehaviorTree/BTFunctionLibrary.h"
#include "BlackboardKeyType_GameplayTag.h"
#include "GameplayBehavior.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayBehaviorsBlueprintFunctionLibrary)

bool UGameplayBehaviorsBlueprintFunctionLibrary::StopGameplayBehavior(TSubclassOf<UGameplayBehavior> GameplayBehaviorClass, AActor* Avatar)
{
	if (Avatar == nullptr || !GameplayBehaviorClass)
	{
		return false;
	}

	return false;
}

FGameplayTagContainer UGameplayBehaviorsBlueprintFunctionLibrary::GetBlackboardValueAsGameplayTag(UBTNode* NodeOwner, const FBlackboardKeySelector& Key)
{
	UBlackboardComponent* BlackboardComp = UBTFunctionLibrary::GetOwnersBlackboard(NodeOwner);
	return GetBlackboardValueAsGameplayTagFromBlackboardComp(BlackboardComp, Key.SelectedKeyName);
}

void UGameplayBehaviorsBlueprintFunctionLibrary::SetBlackboardValueAsGameplayTag(UBTNode* NodeOwner, const FBlackboardKeySelector& Key, FGameplayTagContainer Value)
{
	UBlackboardComponent* BlackboardComp = UBTFunctionLibrary::GetOwnersBlackboard(NodeOwner);
	SetValueAsGameplayTagForBlackboardComp(BlackboardComp, Key.SelectedKeyName, Value);
}

void UGameplayBehaviorsBlueprintFunctionLibrary::AddGameplayTagFilterToBlackboardKeySelector(FBlackboardKeySelector& InSelector, UObject* Owner, FName PropertyName)
{
	const FString FilterName = PropertyName.ToString() + TEXT("_GameplayTag");
	InSelector.AllowedTypes.Add(NewObject<UBlackboardKeyType_GameplayTag>(Owner, *FilterName));
}

FGameplayTagContainer UGameplayBehaviorsBlueprintFunctionLibrary::GetBlackboardValueAsGameplayTagFromBlackboardComp(UBlackboardComponent* BlackboardComp, const FName& KeyName)
{
	return BlackboardComp ? BlackboardComp->GetValue<UBlackboardKeyType_GameplayTag>(KeyName) : FGameplayTagContainer::EmptyContainer;
}


void UGameplayBehaviorsBlueprintFunctionLibrary::SetValueAsGameplayTagForBlackboardComp(UBlackboardComponent* BlackboardComp, const FName& KeyName, FGameplayTagContainer GameplayTagValue)
{
	if (BlackboardComp)
	{
		const FBlackboard::FKey KeyID = BlackboardComp->GetKeyID(KeyName);
		BlackboardComp->SetValue<UBlackboardKeyType_GameplayTag>(KeyID, GameplayTagValue);
	}
}

