// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayBehaviorsModule.h"

#define LOCTEXT_NAMESPACE "GameplayBehaviors"

class FGameplayBehaviorsModule : public IGameplayBehaviorsModule
{
	// Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

IMPLEMENT_MODULE(FGameplayBehaviorsModule, GameplayBehaviorsModule)

void FGameplayBehaviorsModule::StartupModule()
{
	// Called right after the module DLL has been loaded and the module object has been created
	// Load dependent modules here, and they will be guaranteed to be available during ShutdownModule. ie:
	// 		FModuleManager::Get().LoadModuleChecked(TEXT("HTTP"));
}

void FGameplayBehaviorsModule::ShutdownModule()
{
	// Called before the module is unloaded, right before the module object is destroyed.
	// During normal shutdown, this is called in reverse order that modules finish StartupModule().
	// This means that, as long as a module references dependent modules in it's StartupModule(), it
	// can safely reference those dependencies in ShutdownModule() as well.
}

#undef LOCTEXT_NAMESPACE
