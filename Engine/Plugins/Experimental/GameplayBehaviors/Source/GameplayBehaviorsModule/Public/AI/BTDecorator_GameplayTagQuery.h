// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "BehaviorTree/BTDecorator.h"

#include "BTDecorator_GameplayTagQuery.generated.h"

#define UE_API GAMEPLAYBEHAVIORSMODULE_API

class UBehaviorTree;
class UBehaviorTreeComponent;
class UBlackboardComponent;
class UAbilitySystemComponent;

/**
 * GameplayTagQuery decorator node.
 * A decorator node that bases its condition on matching a gameplay tag query.
 */

UCLASS(MinimalAPI, HideCategories=(Condition))
class UBTDecorator_GameplayTagQuery : public UBTDecorator
{
	GENERATED_BODY()
	UBTDecorator_GameplayTagQuery(const FObjectInitializer& ObjectInitializer);

	UE_API virtual bool CalculateRawConditionValue(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) const override;

	/** Callback for when a tag in our query changes */
	void OnGameplayTagInQueryChanged(const FGameplayTag InTag, int32 NewCount, TWeakObjectPtr<UBehaviorTreeComponent> BehaviorTreeComponent);

	UE_API virtual FString GetStaticDescription() const override;

	UE_API virtual void InitializeMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryInit::Type InitType) const override;
	UE_API virtual void CleanupMemory(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory, EBTMemoryClear::Type CleanupType) const override;

protected:

	UPROPERTY(EditAnywhere, Category=GameplayTagQuery, Meta=(ToolTips="Which Actor (from the blackboard) should be checked for this gameplay tag query?"))
	FBlackboardKeySelector ActorForGameplayTagQuery;

	/** Gameplay tag query to match */
	UPROPERTY(EditAnywhere, Category=GameplayTagQuery)
	FGameplayTagQuery GameplayTagQuery;

	UPROPERTY()
	TArray<FGameplayTag> QueryTags;

	/** called when execution flow controller becomes active */
	UE_API virtual void OnBecomeRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;

	/** called when execution flow controller becomes inactive */
	UE_API virtual void OnCeaseRelevant(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;

	UE_API virtual uint16 GetInstanceMemorySize() const override;

#if WITH_EDITOR
	/** Get the array of tags onto which we need to add delegates 
		The gameplay tag interface forces us to allocate everytime we 
		want to know the list of gameplay tags inside a query so we 
		do this only once and cache it.
	*/
	UE_API virtual void CacheGameplayTagsInsideQuery();
	UE_API virtual FString GetErrorMessage() const override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	UE_API virtual void InitializeFromAsset(UBehaviorTree& Asset) override;
};

#undef UE_API
