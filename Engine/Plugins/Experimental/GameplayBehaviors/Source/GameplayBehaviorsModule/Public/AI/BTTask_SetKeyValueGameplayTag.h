// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "BehaviorTree/Tasks/BTTask_BlackboardBase.h"
#include "AI/ValueOrBBKey_GameplayTag.h"

#include "BTTask_SetKeyValueGameplayTag.generated.h"

UCLASS(DisplayName = "Set GameplayTag Key")
class UBTTask_SetKeyValueGameplayTag : public UBTTask_BlackboardBase
{
	GENERATED_BODY()

public:
	UBTTask_SetKeyValueGameplayTag(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	virtual EBTNodeResult::Type ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	virtual FString GetStaticDescription() const override;

private:
	UPROPERTY(EditAnywhere, Category = Blackboard)
	FValueOrBBKey_GameplayTagContainer Value;
};