// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BehaviorTree/BTTaskNode.h"
#include "BTTask_StopGameplayBehavior.generated.h"

#define UE_API GAMEPLAYBEHAVIORSMODULE_API


class UGameplayBehavior;
/**
*
*/
UCLASS(MinimalAPI)
class UBTTask_StopGameplayBehavior : public UBTTaskNode
{
	GENERATED_BODY()
public:
	UE_API UBTTask_StopGameplayBehavior(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	UE_API virtual EBTNodeResult::Type ExecuteTask(UBehaviorTreeComponent& OwnerComp, uint8* NodeMemory) override;
	virtual uint16 GetInstanceMemorySize() const override { return 0; }

	UE_API virtual FString GetStaticDescription() const override;

protected:
	/** If None (the default) will stop any and all gameplay behaviors instigated by the agent*/
	UPROPERTY(EditAnywhere, Category = "Node")
	TSubclassOf<UGameplayBehavior> BehaviorToStop;
};

#undef UE_API
