// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayBehaviorConfig.h"
#include "UObject/Package.h"
#include "GameplayBehaviorConfig_BehaviorTree.generated.h"

#define UE_API GAMEPLAYBEHAVIORSMODULE_API

class UBehaviorTree;


UCLASS(MinimalAPI)
class UGameplayBehaviorConfig_BehaviorTree : public UGameplayBehaviorConfig
{
	GENERATED_BODY()
public:
	UE_API UGameplayBehaviorConfig_BehaviorTree(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	UE_API UBehaviorTree* GetBehaviorTree() const;
	bool ShouldStorePreviousBT() const { return bRevertToPreviousBTOnFinish; }

protected:
	UPROPERTY(EditAnywhere, Category = SmartObject)
	mutable TSoftObjectPtr<UBehaviorTree> BehaviorTree;

	UPROPERTY(EditAnywhere, Category = SmartObject)
	uint32 bRevertToPreviousBTOnFinish : 1;
};

#undef UE_API
