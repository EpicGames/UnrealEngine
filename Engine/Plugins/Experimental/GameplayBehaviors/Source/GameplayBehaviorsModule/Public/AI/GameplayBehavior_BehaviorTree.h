// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/TimerHandle.h"
#include "GameplayBehavior.h"
#include "GameplayBehavior_BehaviorTree.generated.h"

#define UE_API GAMEPLAYBEHAVIORSMODULE_API

class UBehaviorTree;
class AAIController;

/** NOTE: this behavior works only for AIControlled pawns */
UCLASS(MinimalAPI)
class UGameplayBehavior_BehaviorTree : public UGameplayBehavior
{
	GENERATED_BODY()
public:
	UE_API UGameplayBehavior_BehaviorTree(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

protected:
	UE_API virtual bool Trigger(AActor& InAvatar, const UGameplayBehaviorConfig* Config = nullptr, AActor* SmartObjectOwner = nullptr) override;
	UE_API virtual void EndBehavior(AActor& InAvatar, const bool bInterrupted) override;
	UE_API virtual bool NeedsInstance(const UGameplayBehaviorConfig* Config) const override;

	UE_API void OnTimerTick();

	UFUNCTION()
	UE_API void OnPossessedPawnChanged(APawn* OldPawn, APawn* NewPawn);

	UPROPERTY()
	TObjectPtr<UBehaviorTree> PreviousBT;

	UPROPERTY()
	TObjectPtr<AAIController> AIController;

	/** Indicates if BehaviorTree should run only once or in loop. */
	UPROPERTY(EditAnywhere, Category = SmartObject)
	bool bSingleRun = true;
	
	FTimerHandle TimerHandle;
};

#undef UE_API
