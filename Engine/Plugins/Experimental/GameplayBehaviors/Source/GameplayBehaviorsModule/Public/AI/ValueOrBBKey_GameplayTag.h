// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BehaviorTree/ValueOrBBKey.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "ValueOrBBKey_GameplayTag.generated.h"

USTRUCT(BlueprintType)
struct FValueOrBBKey_GameplayTagContainer : public FValueOrBlackboardKeyBase
{
	GENERATED_BODY()

	FValueOrBBKey_GameplayTagContainer(FGameplayTagContainer Default = FGameplayTagContainer())
		: DefaultValue(MoveTemp(Default)) {}
	GAMEPLAYBEHAVIORSMODULE_API FGameplayTagContainer GetValue(const UBehaviorTreeComponent& BehaviorComp) const;
	GAMEPLAYBEHAVIORSMODULE_API FGameplayTagContainer GetValue(const UBehaviorTreeComponent* BehaviorComp) const;
	GAMEPLAYBEHAVIORSMODULE_API FGameplayTagContainer GetValue(const UBlackboardComponent& Blackboard) const;
	GAMEPLAYBEHAVIORSMODULE_API FGameplayTagContainer GetValue(const UBlackboardComponent* Blackboard) const;

	bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

#if WITH_EDITOR
	GAMEPLAYBEHAVIORSMODULE_API virtual bool IsCompatibleType(const UBlackboardKeyType* KeyType) const override;
#endif // WITH_EDITOR

	GAMEPLAYBEHAVIORSMODULE_API virtual FString ToString() const override;

protected:
	UPROPERTY(EditAnywhere, Category = "Value")
	FGameplayTagContainer DefaultValue;
};

UCLASS()
class UValueOrBBKey_GameplayTagBlueprintUtility : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:
	UFUNCTION(BlueprintPure, Category = Blackboard)
	static FGameplayTagContainer GetTagContainer(const FValueOrBBKey_GameplayTagContainer& Value, const UBehaviorTreeComponent* BehaviorTreeComp);
};

template<>
struct TStructOpsTypeTraits<FValueOrBBKey_GameplayTagContainer> : public TStructOpsTypeTraitsBase2<FValueOrBBKey_GameplayTagContainer>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true,
	};
};
