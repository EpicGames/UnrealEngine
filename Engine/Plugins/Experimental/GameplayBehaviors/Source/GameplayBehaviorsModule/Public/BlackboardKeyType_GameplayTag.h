// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "BehaviorTree/Blackboard/BlackboardKeyEnums.h"
#include "BehaviorTree/Blackboard/BlackboardKeyType.h"
#include "GameplayTagContainer.h"
#include "BlackboardKeyType_GameplayTag.generated.h"

#define UE_API GAMEPLAYBEHAVIORSMODULE_API

class UBlackboardComponent;

UCLASS(MinimalAPI, EditInlineNew, meta = (DisplayName = "Gameplay Tag"))
class UBlackboardKeyType_GameplayTag : public UBlackboardKeyType
{
	GENERATED_BODY()
public:

	UE_API UBlackboardKeyType_GameplayTag(const FObjectInitializer& ObjectInitializer);

	typedef FGameplayTagContainer FDataType;
	static UE_API const FDataType InvalidValue;

	static UE_API FGameplayTagContainer GetValue(const UBlackboardKeyType_GameplayTag* KeyOb, const uint8* RawData);
	static UE_API bool SetValue(UBlackboardKeyType_GameplayTag* KeyOb, uint8* RawData, const FGameplayTagContainer& Value);

	UE_API virtual EBlackboardCompare::Type CompareValues(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock,
		const UBlackboardKeyType* OtherKeyOb, const uint8* OtherMemoryBlock) const override;

	UPROPERTY(EditDefaultsOnly, Category = Blackboard)
	FGameplayTagContainer DefaultValue = InvalidValue;

protected:
	UE_API virtual FString DescribeValue(const UBlackboardComponent& OwnerComp, const uint8* RawData) const override;
	UE_API virtual bool TestTextOperation(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock, ETextKeyOperation::Type Op, const FString& OtherString) const override;
	UE_API virtual void CopyValues(UBlackboardComponent& OwnerComp, uint8* MemoryBlock, const UBlackboardKeyType* SourceKeyOb, const uint8* SourceBlock) override;
	UE_API virtual void InitializeMemory(UBlackboardComponent& OwnerComp, uint8* MemoryBlock) override;
	UE_API virtual void Clear(UBlackboardComponent& OwnerComp, uint8* MemoryBlock) override;
	UE_API virtual bool IsEmpty(const UBlackboardComponent& OwnerComp, const uint8* MemoryBlock) const override;
};

#undef UE_API
