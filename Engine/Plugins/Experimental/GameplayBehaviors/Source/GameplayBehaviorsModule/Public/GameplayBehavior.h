// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayTaskOwnerInterface.h"
#include "GameplayTagContainer.h"
#include "GameplayBehavior.generated.h"

#define UE_API GAMEPLAYBEHAVIORSMODULE_API

template <typename OptionalType> struct TOptional;


class AActor;
class APawn;
class ACharacter;
class UGameplayBehaviorConfig;
class UGameplayBehavior;
class UGameplayBehaviorSubsystem;


GAMEPLAYBEHAVIORSMODULE_API DECLARE_LOG_CATEGORY_EXTERN(LogGameplayBehavior, Warning, All);

DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnGameplayBehaviorFinished, UGameplayBehavior& /*Behavior*/, AActor& /*Avatar*/, const bool /*bInterrupted*/)

UENUM()
enum class EGameplayBehaviorInstantiationPolicy : uint8
{
	Instantiate,
	ConditionallyInstantiate,
	DontInstantiate,
};


UCLASS(MinimalAPI, Abstract, Blueprintable, BlueprintType)
class UGameplayBehavior : public UObject, public IGameplayTaskOwnerInterface
{
	GENERATED_BODY()
public:
	UE_API UGameplayBehavior(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	UE_API virtual void PostInitProperties() override;
	UE_API virtual void BeginDestroy() override;
	UE_API virtual UWorld* GetWorld() const override;

	/**
	 * Default implementation will trigger the appropriate Blueprint event based on 
	 * their presence in the script and in the following priority:
	 *  1. OnTriggeredCharacter if used and Avatar is a Character
	 *  2. OnTriggeredPawn if used and Avatar is a Pawn
	 *  3. OnTriggered if used
	 * 
	 * Subclasses can override this method to control the whole flow
	 * @param Avatar The actor on which the behavior should be execute
	 * @param Config Configuration parameters of the behavior
	 * @param SmartObjectOwner The actor associated to the smart object
	 *
	 * @return True if behavior was triggered and caller should register to OnBehaviorFinished to be notified on completion
	 */
	UE_API virtual bool Trigger(AActor& Avatar, const UGameplayBehaviorConfig* Config = nullptr, AActor* SmartObjectOwner = nullptr);
	UE_API virtual void EndBehavior(AActor& Avatar, const bool bInterrupted = false);
	void AbortBehavior(AActor& Avatar) { EndBehavior(Avatar, /*bInterrupted=*/true); }

	FOnGameplayBehaviorFinished& GetOnBehaviorFinishedDelegate() { return OnBehaviorFinished; }

	/** Can return (it's optional) a dynamic location to use this gameplay behavior  */
	UE_API virtual TOptional<FVector> GetDynamicLocation(const AActor* InAvatar = nullptr, const UGameplayBehaviorConfig* InConfig = nullptr, const AActor* InSmartObjectOwner = nullptr) const;

	/**
	 * Returns true if this behavior instance is either a non-CDO object or
	 * is a CDO, but the class is intended to be used via its instances. If
	 * InstantiationPolicy is "ConditionallyInstantiate" then NeedsInstance will
	 * be called to see if class wants to spawn an instance depending on @param Config
	 */
	bool IsInstanced(const UGameplayBehaviorConfig* Config) const
	{
		return (HasAnyFlags(RF_ClassDefaultObject) == false)
			|| InstantiationPolicy == EGameplayBehaviorInstantiationPolicy::Instantiate
			|| (InstantiationPolicy == EGameplayBehaviorInstantiationPolicy::ConditionallyInstantiate
				&& NeedsInstance(Config));
	}

	void SetRelevantActors(const TArray<AActor*>& InRelevantActors) { RelevantActors = InRelevantActors; }
	AActor* GetAvatar() const { return TransientAvatar; }

	// BEGIN IGameplayTaskOwnerInterface
	UE_API virtual UGameplayTasksComponent* GetGameplayTasksComponent(const UGameplayTask& Task) const override;
	UE_API virtual AActor* GetGameplayTaskOwner(const UGameplayTask* Task) const override;
	UE_API virtual AActor* GetGameplayTaskAvatar(const UGameplayTask* Task) const override;
	virtual uint8 GetGameplayTaskDefaultPriority() const override { return FGameplayTasks::ScriptedPriority; }
	UE_API virtual void OnGameplayTaskActivated(UGameplayTask& Task) override;
	UE_API virtual void OnGameplayTaskDeactivated(UGameplayTask& Task) override;
	// END IGameplayTaskOwnerInterface

	//----------------------------------------------------------------------//
	// BP API
	//----------------------------------------------------------------------//

	// @NOTE on trigger functions - we"ll trigger the most specific one that given behavior implements

	UFUNCTION(BlueprintImplementableEvent, Category = GameplayBehavior, meta = (AdvancedDisplay="TagPayload", DisplayName="OnTriggered"))
	UE_API void K2_OnTriggered(AActor* Avatar, const UGameplayBehaviorConfig* Config = nullptr, AActor* SmartObjectOwner = nullptr);

	UFUNCTION(BlueprintImplementableEvent, Category = GameplayBehavior, meta = (AdvancedDisplay = "TagPayload", DisplayName = "OnTriggeredPawn"))
	UE_API void K2_OnTriggeredPawn(APawn* Avatar, const UGameplayBehaviorConfig* Config = nullptr, AActor* SmartObjectOwner = nullptr);

	UFUNCTION(BlueprintImplementableEvent, Category = GameplayBehavior, meta = (AdvancedDisplay = "TagPayload", DisplayName = "OnTriggeredCharacter"))
	UE_API void K2_OnTriggeredCharacter(ACharacter* Avatar, const UGameplayBehaviorConfig* Config = nullptr, AActor* SmartObjectOwner = nullptr);

	UFUNCTION(BlueprintImplementableEvent, Category = GameplayBehavior, meta = (AdvancedDisplay = "TagPayload", DisplayName = "OnFinished"))
	UE_API void K2_OnFinished(AActor* Avatar, bool bWasInterrupted);

	UFUNCTION(BlueprintImplementableEvent, Category = GameplayBehavior, meta = (AdvancedDisplay = "TagPayload", DisplayName = "OnFinishedPawn"))
	UE_API void K2_OnFinishedPawn(APawn* Avatar, bool bWasInterrupted);

	UFUNCTION(BlueprintImplementableEvent, Category = GameplayBehavior, meta = (AdvancedDisplay = "TagPayload", DisplayName = "OnFinishedCharacter"))
	UE_API void K2_OnFinishedCharacter(ACharacter* Avatar, bool bWasInterrupted);

	UFUNCTION(BlueprintCallable, Category = GameplayBehavior, meta = (DisplayName = "EndBehavior"))
	UE_API void K2_EndBehavior(AActor* Avatar);

	UFUNCTION(BlueprintCallable, Category = GameplayBehavior, meta = (DisplayName = "AbortBehavior"))
	UE_API void K2_AbortBehavior(AActor* Avatar);

	UFUNCTION(BlueprintCallable, Category = GameplayBehavior, meta = (DisplayName = "TriggerBehavior"))
	UE_API void K2_TriggerBehavior(AActor* Avatar, UGameplayBehaviorConfig* Config = nullptr, AActor* SmartObjectOwner = nullptr);

	/** @return None if there's no actors or only the one under the index of CurrentIndex is valid */
	UFUNCTION(BlueprintCallable, Category = GameplayBehavior, meta = (DisplayName = "GetNextActorIndexInSequence"))
	UE_API int32 K2_GetNextActorIndexInSequence(int32 CurrentIndex = 0) const;

protected:
	virtual bool NeedsInstance(const UGameplayBehaviorConfig* Config) const { return false; }

protected:
	union
	{
		struct
		{
			uint32 bTriggerGeneric : 1;
			uint32 bTriggerPawn : 1;
			uint32 bTriggerCharacter : 1;
			uint32 bFinishedGeneric : 1;
			uint32 bFinishedPawn : 1;
			uint32 bFinishedCharacter : 1;
		};

		uint32 TransientProps;
	};


	/** */
	uint32 bTransientIsTriggering : 1;
	uint32 bTransientIsActive : 1;
	uint32 bTransientIsEnding : 1;

	EGameplayBehaviorInstantiationPolicy InstantiationPolicy;

	/** Tag identifying behavior this class represents */
	UPROPERTY(EditDefaultsOnly, Category = GameplayBehavior)
	FGameplayTag ActionTag;

	/** Note that it's not going to be called if the behavior finishes as part of Trigger call */
	FOnGameplayBehaviorFinished OnBehaviorFinished;

	/**
	 * It's up to the behavior implementation to decide how to use these actors.
	 * Can be used as patrol points, investigation location, etc.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = GameplayBehavior)
	TArray<TObjectPtr<AActor>> RelevantActors;

	/* SmartObject Actor Owner, can be null */
	UPROPERTY(Transient)
	TObjectPtr<AActor> TransientSmartObjectOwner = nullptr;

	/**
	 * Used mostly as world context for IGameplayTaskOwnerInterface function.
	 *	Use with caution if working with CDOs.
	 *	Set automatically as part of Trigger call
	 */
	UPROPERTY()
	TObjectPtr<AActor> TransientAvatar = nullptr;

private:
	/** List of currently active tasks, do not modify directly */
	UPROPERTY()
	TArray<TObjectPtr<UGameplayTask>> ActiveTasks;
};

#undef UE_API
