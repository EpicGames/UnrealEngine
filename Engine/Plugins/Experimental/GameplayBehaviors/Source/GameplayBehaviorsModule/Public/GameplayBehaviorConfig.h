// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SubclassOf.h"
#include "GameplayBehaviorConfig.generated.h"

#define UE_API GAMEPLAYBEHAVIORSMODULE_API

class UGameplayBehavior;



UCLASS(MinimalAPI, Blueprintable, BlueprintType, EditInlineNew, CollapseCategories)
class UGameplayBehaviorConfig : public UObject
{
	GENERATED_BODY()
public:
	//UGameplayBehavior(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	/** Depending on the specific UGameplayBehavior class returns an instance or CDO of BehaviorClass. */
	UE_API virtual UGameplayBehavior* GetBehavior(UWorld& World) const;

protected:
	UPROPERTY(EditDefaultsOnly, Category = GameplayBehavior)
	TSubclassOf<UGameplayBehavior> BehaviorClass;
};

#undef UE_API
