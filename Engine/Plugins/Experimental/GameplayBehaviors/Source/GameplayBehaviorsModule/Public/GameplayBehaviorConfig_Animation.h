// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimMontage.h"
#include "GameplayBehaviorConfig.h"
#include "GameplayBehaviorConfig_Animation.generated.h"

#define UE_API GAMEPLAYBEHAVIORSMODULE_API


UCLASS(MinimalAPI)
class UGameplayBehaviorConfig_Animation : public UGameplayBehaviorConfig
{
	GENERATED_BODY()
public:
	UE_API UGameplayBehaviorConfig_Animation(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	/** Depending on the specific UGameplayBehavior class returns an instance or CDO of BehaviorClass. */
	//virtual UGameplayBehavior* GetBehavior(UWorld& World) const;

	UAnimMontage* GetMontage() const { return AnimMontage.Get(); }
	float GetPlayRate() const { return PlayRate; }
	FName GetStartSectionName() const { return StartSectionName; }
	bool IsLooped() const { return (bLoop != 0); }

protected:
	/*UPROPERTY(EditDefaultsOnly, Category = GameplayBehavior)
	TSubclassOf<UGameplayBehavior> BehaviorClass;*/
	UPROPERTY(EditAnywhere, Category = SmartObject)
	mutable TSoftObjectPtr<UAnimMontage> AnimMontage;

	UPROPERTY(EditAnywhere, Category = SmartObject)
	float PlayRate = 1.f;

	UPROPERTY(EditAnywhere, Category = SmartObject)
	FName StartSectionName;

	UPROPERTY(EditAnywhere, Category = SmartObject)
	uint32 bLoop : 1;
};

#undef UE_API
