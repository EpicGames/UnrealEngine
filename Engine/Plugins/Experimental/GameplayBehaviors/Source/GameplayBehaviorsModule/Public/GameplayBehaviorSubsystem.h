// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/WorldSubsystem.h"
#include "GameplayBehaviorSubsystem.generated.h"

#define UE_API GAMEPLAYBEHAVIORSMODULE_API

template <typename T> class TSubclassOf;


class UGameplayBehavior;
class AActor;
class UGameplayBehaviorConfig;
class UWorld;

USTRUCT()
struct FAgentGameplayBehaviors
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<TObjectPtr<UGameplayBehavior>> Behaviors;
};

UCLASS(MinimalAPI, config = Game, defaultconfig, Transient)
class UGameplayBehaviorSubsystem : public UWorldSubsystem
{
	GENERATED_BODY()
public:
	static UE_API UGameplayBehaviorSubsystem* GetCurrent(const UWorld* World);
	static UE_API bool TriggerBehavior(const UGameplayBehaviorConfig& Config, AActor& Avatar, AActor* SmartObjectOwner = nullptr);
	static UE_API bool TriggerBehavior(UGameplayBehavior& Behavior, AActor& Avatar, const UGameplayBehaviorConfig* Config, AActor* SmartObjectOwner = nullptr);
	UE_API bool StopBehavior(AActor& Avatar, TSubclassOf<UGameplayBehavior> BehaviorToStop);

protected:
	UE_API void OnBehaviorFinished(UGameplayBehavior& Behavior, AActor& Avatar, const bool bInterrupted);

	UE_API virtual bool TriggerBehaviorImpl(UGameplayBehavior& Behavior, AActor& Avatar, const UGameplayBehaviorConfig* Config, AActor* SmartObjectOwner = nullptr);

	UPROPERTY()
	TMap<TObjectPtr<AActor>, FAgentGameplayBehaviors> AgentGameplayBehaviors;
};

#undef UE_API
