// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TimerManager.h"
#include "GameplayBehavior.h"
#include "GameplayBehavior_AnimationBased.generated.h"

#define UE_API GAMEPLAYBEHAVIORSMODULE_API


class AActor;
class UAnimMontage;
class UAbilitySystemComponent;


USTRUCT()
struct FMontagePlaybackData
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<AActor> Avatar;

	UPROPERTY()
	TObjectPtr<UAnimMontage> AnimMontage;

	UPROPERTY()
	TObjectPtr<UAbilitySystemComponent> AbilityComponent;

	float PlayRate;
	FName SectionName;
	FTimerHandle TimerHandle;
	FTimerDelegate TimerDelegate;
	uint8 bLoop : 1;

	FMontagePlaybackData()
	{
		FMemory::Memzero(*this);
	}

	FMontagePlaybackData(AActor& InAvatar, UAnimMontage& InAnimMontage, const float InPlayRate = 1.f, const FName InSectionName = NAME_None, const bool bInLoop = false)
		: Avatar(&InAvatar), AnimMontage(&InAnimMontage)
		, AbilityComponent(nullptr), PlayRate(InPlayRate)
		, SectionName(InSectionName), bLoop(bInLoop)
	{}
	
	bool operator==(const FMontagePlaybackData& Other) const { return Avatar == Other.Avatar && AnimMontage == Other.AnimMontage; }
	bool operator==(const AActor* InAvatar) const { return Avatar == InAvatar; }
};


/**
 *	Note that this behavior is supporting playing only a single montage for a
 *	given Avatar at a time. Trying to play multiple or using multiple UGameplayBehavior_AnimationBased
 *	instances will result in requests overriding and interfering. */
UCLASS(MinimalAPI)
class UGameplayBehavior_AnimationBased : public UGameplayBehavior
{
	GENERATED_BODY()
public:
	UE_API UGameplayBehavior_AnimationBased(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	UE_API virtual bool Trigger(AActor& InAvatar, const UGameplayBehaviorConfig* Config = nullptr, AActor* SmartObjectOwner = nullptr) override;
	UE_API virtual void EndBehavior(AActor& InAvatar, const bool bInterrupted) override;

	UE_API bool PlayMontage(AActor& InAvatar, UAnimMontage& AnimMontage, const float InPlayRate = 1.f, const FName StartSectionName = NAME_None, const bool bLoop = false);

protected:
	UFUNCTION()
	UE_API void OnMontageFinished(UAnimMontage* Montage, bool bInterrupted, AActor* InAvatar);

	/**
	 *	If this array ever gets more than couple elements at a time we should consider
	 *	switching over to a TMap
	 */ 
	UPROPERTY()
	mutable TArray<FMontagePlaybackData> ActivePlayback;
};

#undef UE_API
