// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "GameplayBehaviorsBlueprintFunctionLibrary.generated.h"

#define UE_API GAMEPLAYBEHAVIORSMODULE_API

class UBlackboardComponent;
class UGameplayBehavior;
struct FBlackboardKeySelector;
struct FGameplayTagContainer;
template <typename T> class TSubclassOf;


class AActor;
class UBTNode;

UCLASS(MinimalAPI, meta = (ScriptName = "GameplayBehaviorLibrary"))
class UGameplayBehaviorsBlueprintFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:
	/** Will force-stop GameplayBehavior on given Avatar assuming the current 
	 *	behavior is of GameplayBehaviorClass class*/
	static UE_API bool StopGameplayBehavior(TSubclassOf<UGameplayBehavior> GameplayBehaviorClass,  AActor* Avatar);

	UFUNCTION(BlueprintPure, Category = "AI|BehaviorTree", Meta = (HidePin = "NodeOwner", DefaultToSelf = "NodeOwner"))
	static UE_API FGameplayTagContainer GetBlackboardValueAsGameplayTag(UBTNode* NodeOwner, const FBlackboardKeySelector& Key);

	UFUNCTION(BlueprintCallable, Category = "AI|BehaviorTree", Meta = (HidePin = "NodeOwner", DefaultToSelf = "NodeOwner"))
	static UE_API void SetBlackboardValueAsGameplayTag(UBTNode* NodeOwner, const FBlackboardKeySelector& Key, FGameplayTagContainer Value);

	UFUNCTION(BlueprintCallable, Category = "AI|BehaviorTree")
	static UE_API void AddGameplayTagFilterToBlackboardKeySelector(FBlackboardKeySelector& InSelector, UObject* Owner, FName PropertyName);

	UFUNCTION(BlueprintCallable, Category = "AI|BehaviorTree")
	static UE_API FGameplayTagContainer GetBlackboardValueAsGameplayTagFromBlackboardComp(UBlackboardComponent* BlackboardComp, const FName& KeyName);

	UFUNCTION(BlueprintCallable, Category = "AI|BehaviorTree")
	static UE_API void SetValueAsGameplayTagForBlackboardComp(UBlackboardComponent* BlackboardComp, const FName& KeyName, FGameplayTagContainer GameplayTagValue);

};

#undef UE_API
