// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

namespace UnrealBuildTool.Rules
{
    public class GameplayBehaviorsTestSuite : ModuleRules
    {
        public GameplayBehaviorsTestSuite(ReadOnlyTargetRules Target) : base(Target)
		{
            PublicIncludePaths.AddRange(
                    new string[] {
                    }
                    );

            PublicDependencyModuleNames.AddRange(
                new string[] {
                        "Core",
                        "CoreUObject",
                        "Engine",
                        "AIModule",
                        "GameplayBehaviorsModule",
                        "AITestSuite",
                }
                );

            if (Target.bBuildEditor == true)
            {
				PrivateDependencyModuleNames.Add("EditorFramework");
                PrivateDependencyModuleNames.Add("UnrealEd");
            }
        }
    }
}